// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioFormatOgg : ModuleRules
{
	protected virtual bool bWithOggVorbis { get => (
			(Target.Platform == UnrealTargetPlatform.Win64) ||
			(Target.Platform == UnrealTargetPlatform.Mac) ||
			Target.IsInPlatformGroup(UnrealPlatformGroup.Linux)
		);
	}

	public AudioFormatOgg(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("TargetPlatform");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
				"VorbisAudioDecoder"
			}
		);

		if (bWithOggVorbis)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"UEOgg",
				"Vorbis",
				"VorbisFile"
			);
		}
	}
}
