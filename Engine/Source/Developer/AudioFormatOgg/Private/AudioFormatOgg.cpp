// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioFormatOgg.h"
#include "Serialization/MemoryWriter.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IAudioFormat.h"
#include "Interfaces/IAudioFormatModule.h"
#include "Decoders/VorbisAudioInfo.h"

#if WITH_OGGVORBIS
	#pragma pack(push, 8)
	#include "vorbis/vorbisenc.h"
	#include "vorbis/vorbisfile.h"
	#pragma pack(pop)
#endif

static_assert(WITH_OGGVORBIS, "No point in compiling the OGG compressor if we don't have Vorbis.");

// Vorbis encoded sound is about 15% better quality than XMA - adjust the quality setting to get consistent cross platform sound quality
#define VORBIS_QUALITY_MODIFIER		0.85

#define SAMPLES_TO_READ		1024
#define SAMPLE_SIZE			( ( uint32 )sizeof( short ) )

static FName NAME_OGG(TEXT("OGG"));
/**
 * IAudioFormat, audio compression abstraction
**/
class FAudioFormatOgg : public IAudioFormat
{
	enum
	{
		/** Version for OGG format, this becomes part of the DDC key. */
		UE_AUDIO_OGG_VER = 6,
	};

public:
	virtual bool AllowParallelBuild() const override
	{
		return true;
	}

	virtual uint16 GetVersion(FName Format) const override
	{
		check(Format == NAME_OGG);
		return UE_AUDIO_OGG_VER;
	}


	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Add(NAME_OGG);
	}

	virtual bool Cook(FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& CompressedDataStore) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatOgg::Cook);
		check(Format == NAME_OGG);
#if WITH_OGGVORBIS
		{

			short				ReadBuffer[SAMPLES_TO_READ * SAMPLE_SIZE * 2];

			ogg_stream_state	os;		// take physical pages, weld into a logical stream of packets 
			ogg_page			og;		// one ogg bitstream page.  Vorbis packets are inside
			ogg_packet			op;		// one raw packet of data for decode
			vorbis_info			vi;		// struct that stores all the static vorbis bitstream settings
			vorbis_comment		vc;		// struct that stores all the user comments
			vorbis_dsp_state	vd;		// central working state for the packet->PCM decoder
			vorbis_block		vb;		// local working space for packet->PCM decode
			uint32				i;
			bool				eos;

			// Create a buffer to store compressed data
			CompressedDataStore.Empty();
			FMemoryWriter CompressedData( CompressedDataStore );
			uint32 BufferOffset = 0;

			float CompressionQuality = ( float )( QualityInfo.Quality * VORBIS_QUALITY_MODIFIER ) / 100.0f;
			CompressionQuality = FMath::Clamp( CompressionQuality, -0.1f, 1.0f );

			vorbis_info_init( &vi );

			if( vorbis_encode_init_vbr( &vi, QualityInfo.NumChannels, QualityInfo.SampleRate, CompressionQuality ) )
			{
				return false;
			}

			// add a comment
			vorbis_comment_init( &vc );
			vorbis_comment_add_tag( &vc, "ENCODER", "UnrealEngine4" );

			// set up the analysis state and auxiliary encoding storage
			vorbis_analysis_init( &vd, &vi );
			vorbis_block_init( &vd, &vb );

			// set up our packet->stream encoder
			ogg_stream_init( &os, 0 );

			ogg_packet header;
			ogg_packet header_comm;
			ogg_packet header_code;

			vorbis_analysis_headerout( &vd, &vc, &header, &header_comm, &header_code);
			ogg_stream_packetin( &os, &header );
			ogg_stream_packetin( &os, &header_comm );
			ogg_stream_packetin( &os, &header_code );

			// This ensures the actual audio data will start on a new page, as per spec
			while( true )
			{
				int result = ogg_stream_flush( &os, &og );
				if( result == 0 )
				{
					break;
				}

				CompressedData.Serialize( og.header, og.header_len );
				CompressedData.Serialize( og.body, og.body_len );
			}

			eos = false;
			while( !eos )
			{
				// Read samples
				uint32 BytesToRead = FMath::Min( SAMPLES_TO_READ * QualityInfo.NumChannels * SAMPLE_SIZE, QualityInfo.SampleDataSize - BufferOffset );
				FMemory::Memcpy( ReadBuffer, SrcBuffer.GetData() + BufferOffset, BytesToRead );
				BufferOffset += BytesToRead;

				if( BytesToRead == 0)
				{
					// end of file
					vorbis_analysis_wrote( &vd, 0 );
				}
				else
				{
					// expose the buffer to submit data
					float **buffer = vorbis_analysis_buffer( &vd, SAMPLES_TO_READ );

					if( QualityInfo.NumChannels == 1 )
					{
						for( i = 0; i < BytesToRead / SAMPLE_SIZE; i++ )
						{
							buffer[0][i] = ( ReadBuffer[i] ) / 32768.0f;
						}
					}
					else
					{
						for( i = 0; i < BytesToRead / ( SAMPLE_SIZE * 2 ); i++ )
						{
							buffer[0][i] = ( ReadBuffer[i * 2] ) / 32768.0f;
							buffer[1][i] = ( ReadBuffer[i * 2 + 1] ) / 32768.0f;
						}
					}

					// tell the library how many samples we actually submitted
					vorbis_analysis_wrote( &vd, i );
				}

				// vorbis does some data preanalysis, then divvies up blocks for more involved (potentially parallel) processing.
				while( vorbis_analysis_blockout( &vd, &vb ) == 1 )
				{
					// analysis, assume we want to use bitrate management
					vorbis_analysis( &vb, NULL );
					vorbis_bitrate_addblock( &vb );

					while( vorbis_bitrate_flushpacket( &vd, &op ) )
					{
						// weld the packet into the bitstream
						ogg_stream_packetin( &os, &op );

						// write out pages (if any)
						while( !eos )
						{
							int result = ogg_stream_pageout( &os, &og );
							if( result == 0 )
							{
								break;
							}
							CompressedData.Serialize( og.header, og.header_len );
							CompressedData.Serialize( og.body, og.body_len );

							// this could be set above, but for illustrative purposes, I do	it here (to show that vorbis does know where the stream ends)
							if( ogg_page_eos( &og ) )
							{
								eos = true;
							}
						}
					}
				}
			}

			// clean up and exit.  vorbis_info_clear() must be called last
			ogg_stream_clear( &os );
			vorbis_block_clear( &vb );
			vorbis_dsp_clear( &vd );
			vorbis_comment_clear( &vc );
			vorbis_info_clear( &vi );
			// ogg_page and ogg_packet structs always point to storage in libvorbis.  They're never freed or manipulated directly
		}
		return CompressedDataStore.Num() > 0;
#else
		return false;
#endif		// WITH_OGGVOBVIS
	}

	virtual bool CookSurround(FName Format, const TArray<TArray<uint8> >& SrcBuffers, FSoundQualityInfo& QualityInfo, TArray<uint8>& CompressedDataStore) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatOgg::CookSurround);
		check(Format == NAME_OGG);
#if WITH_OGGVORBIS
		{

			ogg_stream_state	os;		// take physical pages, weld into a logical stream of packets 
			ogg_page			og;		// one ogg bitstream page.  Vorbis packets are inside
			ogg_packet			op;		// one raw packet of data for decode
			vorbis_info			vi;		// struct that stores all the static vorbis bitstream settings
			vorbis_comment		vc;		// struct that stores all the user comments
			vorbis_dsp_state	vd;		// central working state for the packet->PCM decoder
			vorbis_block		vb;		// local working space for packet->PCM decode
			bool				eos;
			int					j;

			// Create a buffer to store compressed data
			CompressedDataStore.Empty();
			FMemoryWriter CompressedData( CompressedDataStore );
			uint32 BufferOffset = 0;

			int32 Size = -1;
			for (int32 Index = 0; Index < SrcBuffers.Num(); Index++)
			{
				if (!Index)
				{
					Size = SrcBuffers[Index].Num();
				}
				else
				{
					if (Size != SrcBuffers[Index].Num())
					{
						return false;
					}
				}
			}
			if (Size <= 0)
			{
				return false;
			}

			// Extract the relevant info for compression
			float CompressionQuality = ( float )( QualityInfo.Quality * VORBIS_QUALITY_MODIFIER ) / 100.0f;
			CompressionQuality = FMath::Clamp( CompressionQuality, 0.0f, 1.0f );

			vorbis_info_init( &vi );

			if( vorbis_encode_init_vbr( &vi, SrcBuffers.Num(), QualityInfo.SampleRate, CompressionQuality ) )
			{
				return false;
			}

			// add a comment
			vorbis_comment_init( &vc );
			vorbis_comment_add_tag( &vc, "ENCODER", "UnrealEngine4" );

			// set up the analysis state and auxiliary encoding storage
			vorbis_analysis_init( &vd, &vi );
			vorbis_block_init( &vd, &vb );

			// set up our packet->stream encoder
			ogg_stream_init( &os, 0 );

			ogg_packet header;
			ogg_packet header_comm;
			ogg_packet header_code;

			vorbis_analysis_headerout( &vd, &vc, &header, &header_comm, &header_code);
			ogg_stream_packetin( &os, &header );
			ogg_stream_packetin( &os, &header_comm );
			ogg_stream_packetin( &os, &header_code );

			// This ensures the actual audio data will start on a new page, as per spec
			while( true )
			{
				int result = ogg_stream_flush( &os, &og );
				if( result == 0 )
				{
					break;
				}

				CompressedData.Serialize( og.header, og.header_len );
				CompressedData.Serialize( og.body, og.body_len );
			}

			TArray<int32> ChannelOrder = GetChannelOrder(SrcBuffers.Num());

			eos = false;
			while( !eos )
			{
				// Read samples
				uint32 BytesToRead = FMath::Min( SAMPLES_TO_READ * SAMPLE_SIZE, Size - BufferOffset );
				if( BytesToRead == 0)
				{
					// end of file
					vorbis_analysis_wrote( &vd, 0 );
				}
				else
				{
					// expose the buffer to submit data
					float **buffer = vorbis_analysis_buffer( &vd, SAMPLES_TO_READ );

					uint32 i = 0;
					for( j = 0; j < ChannelOrder.Num(); j++ )
					{
						short* ReadBuffer = (short*)(SrcBuffers[ChannelOrder[j]].GetData() + BufferOffset);
						for( i = 0; i < BytesToRead / SAMPLE_SIZE; i++ )
						{
							buffer[j][i] = ( ReadBuffer[i] ) / 32768.0f;
						}
					}

					// tell the library how many samples we actually submitted
					vorbis_analysis_wrote( &vd, i );
				}
				BufferOffset += BytesToRead;

				// vorbis does some data preanalysis, then divvies up blocks for more involved (potentially parallel) processing.
				while( vorbis_analysis_blockout( &vd, &vb ) == 1 )
				{
					// analysis, assume we want to use bitrate management
					vorbis_analysis( &vb, NULL );
					vorbis_bitrate_addblock( &vb );

					while( vorbis_bitrate_flushpacket( &vd, &op ) )
					{
						// weld the packet into the bitstream
						ogg_stream_packetin( &os, &op );

						// write out pages (if any)
						while( !eos )
						{
							int result = ogg_stream_pageout( &os, &og );
							if( result == 0 )
							{
								break;
							}
							CompressedData.Serialize( og.header, og.header_len );
							CompressedData.Serialize( og.body, og.body_len );

							// this could be set above, but for illustrative purposes, I do	it here (to show that vorbis does know where the stream ends)
							if( ogg_page_eos( &og ) )
							{
								eos = true;
							}
						}
					}
				}
			}

			// clean up and exit.  vorbis_info_clear() must be called last
			ogg_stream_clear( &os );
			vorbis_block_clear( &vb );
			vorbis_dsp_clear( &vd );
			vorbis_comment_clear( &vc );
			vorbis_info_clear( &vi );

			// ogg_page and ogg_packet structs always point to storage in libvorbis.  They're never freed or manipulated directly
		}
		return CompressedDataStore.Num() > 0;
#else
		return false;
#endif		// WITH_OGGVOBVIS
	}

	/**
	 * Put channels into the order expected for a multi-channel ogg vorbis file.
	 * Ordering taken from http://xiph.org/vorbis/doc/Vorbis_I_spec.html#x1-800004.3.9
	 * Only concerned with 6 channel version because that seems to apply filtering to LFE.
	 */
	TArray<int32> GetChannelOrder(int32 NumChannels) const
	{
		TArray<int32> ChannelOrder;

		switch(NumChannels)
		{
			case 6:
			{
				// the stream is 5.1 surround. channel order: front left, center, front right, rear left, rear right, LFE
				for (int32 i = 0; i < NumChannels; i++)
				{
					ChannelOrder.Add(VorbisChannelInfo::Order[NumChannels - 1][i]);
				}
				break;
			}

			default:
			{
				// no special ordering, just put all channels into ordered buffer
				for( int32 i = 0; i < NumChannels; i++ )
				{
					ChannelOrder.Add(i);
				}
				break;
			}
		}

		return ChannelOrder;
	}

	virtual int32 Recompress(FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& OutBuffer) const override
	{
		check(Format == NAME_OGG);
		FVorbisAudioInfo	AudioInfo;
		int32				CompressedSize = -1;

		// Cannot quality preview multichannel sounds
		if( QualityInfo.NumChannels > 2 )
		{
			return 0;
		}
		TArray<uint8> CompressedDataStore;
		if( !Cook( Format, SrcBuffer, QualityInfo, CompressedDataStore ) )
		{
			return 0;
		}

		// Parse the audio header for the relevant information
		if (!AudioInfo.ReadCompressedInfo(CompressedDataStore.GetData(), CompressedDataStore.Num(), &QualityInfo))
		{
			return 0;
		}

		// Decompress all the sample data
		OutBuffer.Empty(QualityInfo.SampleDataSize);
		OutBuffer.AddZeroed(QualityInfo.SampleDataSize);
		AudioInfo.ExpandFile(OutBuffer.GetData(), &QualityInfo);

		return CompressedDataStore.Num();
	}

	virtual int32 GetMinimumSizeForInitialChunk(FName Format, const TArray<uint8>& SrcBuffer) const override
	{
		// This is the old hard-coded size for the Zeroth chunk. 
		static const int32 OldDefaultSizeForBackwardsCompat = 6 * 1024;

		uint8 const* SrcData = SrcBuffer.GetData();
		uint32 SrcSize = SrcBuffer.Num();

		FVorbisAudioInfo	AudioInfo;
		FSoundQualityInfo	QualityInfo;
		if (!AudioInfo.ReadCompressedInfo((uint8*)SrcData, SrcSize, &QualityInfo))
		{
			return OldDefaultSizeForBackwardsCompat;
		}

		// Ask where the audio data starts, this is our minimum size, which includes the minimum of the 3 necessary headers. 
		// This is the true minimum size for the Zeroth chunk.
		int32 DataOffset = AudioInfo.GetAudioDataStartOffset();
		if (DataOffset > 0)
		{
			// For backwards compatibility only increase the size if we need to, which should prevent a large delta.
			return FMath::Max(OldDefaultSizeForBackwardsCompat, DataOffset);
		}

		return OldDefaultSizeForBackwardsCompat;
	}

	virtual bool SplitDataForStreaming(const TArray<uint8>& SrcBuffer, TArray<TArray<uint8>>& OutBuffers, const int32 InitialChunkMaxSize, const int32 MaxChunkSize) const override
	{
		// Just chunk purely on MONO_PCM_BUFFER_SIZE
		uint8 const*	SrcData = SrcBuffer.GetData();
		uint32			SrcSize = SrcBuffer.Num();

		// Load the audio quality info to get the number of channels
		FVorbisAudioInfo	AudioInfo;
		FSoundQualityInfo	QualityInfo;
		if (!AudioInfo.ReadCompressedInfo((uint8*)SrcData, SrcSize, &QualityInfo))
		{
			return false;
		}


		// Set up initial chunk:
		uint32 ChunkSize = InitialChunkMaxSize;
		uint32	CurChunkDataSize = FMath::Min<uint32>(SrcSize, ChunkSize);

		AddNewChunk(OutBuffers, CurChunkDataSize);
		AddChunkData(OutBuffers, SrcData, CurChunkDataSize);

		check(SrcSize >= CurChunkDataSize);
		SrcSize -= CurChunkDataSize;
		SrcData += CurChunkDataSize;

		ChunkSize = MaxChunkSize;

		// Set up the rest of the chunks:
		while(SrcSize > 0)
		{
			CurChunkDataSize = FMath::Min<uint32>(SrcSize, ChunkSize);

			AddNewChunk(OutBuffers, CurChunkDataSize);
			AddChunkData(OutBuffers, SrcData, CurChunkDataSize);

			check(SrcSize >= CurChunkDataSize);
			SrcSize -= CurChunkDataSize;
			SrcData += CurChunkDataSize;
		}

		return true;
	}
	
	// Add a new chunk and reserve ChunkSize bytes in it
	void AddNewChunk(TArray<TArray<uint8>>& OutBuffers, int32 ChunkReserveSize) const
	{
		TArray<uint8>& NewBuffer = OutBuffers.AddDefaulted_GetRef();
		NewBuffer.Empty(ChunkReserveSize);
	}
	
	// Add data to the current chunk
	void AddChunkData(TArray<TArray<uint8>>& OutBuffers, const uint8* ChunkData, int32 ChunkDataSize) const
	{
		TArray<uint8>& TargetBuffer = OutBuffers[OutBuffers.Num() - 1];
		TargetBuffer.Append(ChunkData, ChunkDataSize);
	}

};


/**
 * Module for ogg audio compression
 */

class FAudioPlatformOggModule : public IAudioFormatModule
{
public:
	virtual IAudioFormat* GetAudioFormat()
	{
		LoadVorbisLibraries();
		static FAudioFormatOgg Instance;
		return &Instance;
	}
};

IMPLEMENT_MODULE( FAudioPlatformOggModule, AudioFormatOgg);
